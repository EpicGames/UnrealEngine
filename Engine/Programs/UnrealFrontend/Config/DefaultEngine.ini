[URL]
GameNameShort=UFE

[Plugins]
+EnabledPlugins="MessagingDebugger"
-EnabledPlugins="ExampleDeviceProfileSelector"
+ProgramEnabledPlugins="UdpMessaging"
+ProgramEnabledPlugins="TcpMessaging"
+ProgramEnabledPlugins="VisualStudioSourceCodeAccess"
+ProgramEnabledPlugins="XcodeSourceCodeAccess"

[SectionsToSave]
bCanSaveAllSections=true
Section=CurrentIniVersion

